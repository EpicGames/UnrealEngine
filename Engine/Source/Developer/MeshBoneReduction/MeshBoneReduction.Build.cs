// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class MeshBoneReduction : ModuleRules
{
    public MeshBoneReduction(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.Add("Core");
		PrivateDependencyModuleNames.Add("CoreUObject");
		PrivateDependencyModuleNames.Add("Engine");
        PrivateDependencyModuleNames.Add("RenderCore");
		PrivateDependencyModuleNames.Add("RHI");
        PrivateDependencyModuleNames.Add("AnimationBlueprintLibrary");
    }
}
