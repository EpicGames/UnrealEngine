// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshBoneReduction.h"
#include "Modules/ModuleManager.h"
#include "GPUSkinPublicDefs.h"
#include "ReferenceSkeleton.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/SkinnedAssetCommon.h"
#include "Components/SkinnedMeshComponent.h"
#include "UObject/UObjectHash.h"
#include "ComponentReregisterContext.h"
#include "Templates/UniquePtr.h"
#include "Rendering/SkeletalMeshModel.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "AnimationBlueprintLibrary.h"
#include "BoneWeights.h"
#include "Async/ParallelFor.h"
#include "Animation/AnimSequence.h"
#include "Animation/AnimSequenceHelpers.h"
#include "Templates/UnrealTemplate.h"

class FMeshBoneReductionModule : public IMeshBoneReductionModule
{
public:
	// IModuleInterface interface.
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	// IMeshBoneReductionModule interface.
	virtual class IMeshBoneReduction* GetMeshBoneReductionInterface() override;
};

DEFINE_LOG_CATEGORY_STATIC(LogMeshBoneReduction, Log, All);
IMPLEMENT_MODULE(FMeshBoneReductionModule, MeshBoneReduction);

class FMeshBoneReduction : public IMeshBoneReduction
{
public:

	virtual ~FMeshBoneReduction()
	{
	}

	void EnsureChildrenPresents(int32 BoneIndex, const TArray<FMeshBoneInfo>& RefBoneInfo, TArray<int32>& OutBoneIndicesToRemove)
	{
		// just look for direct parent, we could look for RefBoneInfo->Ischild, but more expensive, and no reason to do that all the work
		for (int32 ChildBoneIndex = 0; ChildBoneIndex < static_cast<int32>( RefBoneInfo.Num()); ++ChildBoneIndex)
		{
			if (RefBoneInfo[ChildBoneIndex].ParentIndex == BoneIndex)
			{
				OutBoneIndicesToRemove.AddUnique(ChildBoneIndex);
				EnsureChildrenPresents(ChildBoneIndex, RefBoneInfo, OutBoneIndicesToRemove);
			}
		}
	}

	bool GetBoneReductionData(const USkeletalMesh* SkeletalMesh, int32 DesiredLOD, TMap<FBoneIndexType, FBoneIndexType>& OutBonesToReplace, const TArray<FName>* BoneNamesToRemove = nullptr) override
	{
		if (!SkeletalMesh)
		{
			return false;
		}

		if (!SkeletalMesh->IsValidLODIndex(DesiredLOD))
		{
			return false;
		}

		const TArray<FMeshBoneInfo> & RefBoneInfo = SkeletalMesh->GetRefSkeleton().GetRawRefBoneInfo();
		TArray<int32> BoneIndicesToRemove;

		// originally this code was accumulating from LOD 0->DesiredLOd, but that should be done outside of tool if they want to
		// removing it, and just include DesiredLOD
		{
			// if name is entered, use them instead of setting
			const TArray<FName>& BonesToRemoveSetting = [BoneNamesToRemove, SkeletalMesh, DesiredLOD]()
			{
				if (BoneNamesToRemove)
				{
					return *BoneNamesToRemove;
				}
				else
				{
					TArray<FName> RetrievedNames;
					const FSkeletalMeshLODInfo* LODInfo = SkeletalMesh->GetLODInfo(DesiredLOD);
					for (const FBoneReference& BoneReference : LODInfo->BonesToRemove)
					{
						RetrievedNames.AddUnique(BoneReference.BoneName);
					}
					RetrievedNames.Remove(NAME_None);

					return RetrievedNames;
				}
			}();

			// first gather indices. we don't want to add bones to replace if that "to-be-replace" will be removed as well
			for (int32 Index = 0; Index < BonesToRemoveSetting.Num(); ++Index)
			{
				if (BonesToRemoveSetting[Index] != NAME_None)
				{
					int32 BoneIndex = SkeletalMesh->GetRefSkeleton().FindRawBoneIndex(BonesToRemoveSetting[Index]);

					// we don't allow root to be removed
					if (BoneIndex > 0)
					{
						BoneIndicesToRemove.AddUnique(BoneIndex);
						// make sure all children for this joint is included
						EnsureChildrenPresents(BoneIndex, RefBoneInfo, BoneIndicesToRemove);
					}
				}
			
			}
		}

		if (BoneIndicesToRemove.Num() <= 0)
		{
			return false;
		}

		// now make sure the parent isn't the one to be removed, find the one that won't be removed
		for (int32 Index = 0; Index < BoneIndicesToRemove.Num(); ++Index)
		{
			int32 BoneIndex = BoneIndicesToRemove[Index];
			int32 ParentIndex = RefBoneInfo[BoneIndex].ParentIndex;

			while (BoneIndicesToRemove.Contains(ParentIndex))
			{
				ParentIndex = RefBoneInfo[ParentIndex].ParentIndex;
			}

			OutBonesToReplace.Add(IntCastChecked<FBoneIndexType>(BoneIndex), IntCastChecked<FBoneIndexType>(ParentIndex));
		}

		return ( OutBonesToReplace.Num() > 0 );
	}

	void FixUpSectionBoneMaps( FSkelMeshSection & Section, const TMap<FBoneIndexType, FBoneIndexType> &BonesToRepair, TMap<FName, FImportedSkinWeightProfileData>& SkinWeightProfiles) override
	{
		// now you have list of bones, remove them from vertex influences
		{
			// FBoneIndexType/uint16 max range
			const int32 FBoneIndexTypeMax = 65536;
			TMap<FBoneIndexType, FBoneIndexType> BoneMapRemapTable;
			// first go through bone map and see if this contains BonesToRemove
			int32 BoneMapSize = Section.BoneMap.Num();
			int32 AdjustIndex=0;

			for (int32 BoneMapIndex=0; BoneMapIndex < BoneMapSize; ++BoneMapIndex )
			{
				// look for this bone to be removed or not?
				const FBoneIndexType* ParentBoneIndex = BonesToRepair.Find(Section.BoneMap[BoneMapIndex]);
				if ( ParentBoneIndex  )
				{
					// this should not happen, I don't ever remove root
					check (*ParentBoneIndex!=INDEX_NONE);

					// if Parent already exists in the current BoneMap, we just have to fix up the mapping
					int32 ParentBoneMapIndex = Section.BoneMap.Find(*ParentBoneIndex);

					// if it exists
					if (ParentBoneMapIndex != INDEX_NONE)
					{
						// if parent index is higher, we have to decrease it to match to new index
						if (ParentBoneMapIndex > BoneMapIndex)
						{
							--ParentBoneMapIndex;
						}

						// remove current section count, will replace with parent
						Section.BoneMap.RemoveAt(BoneMapIndex);
					}
					else
					{
						// if parent doesn't exist, we have to add one
						// this doesn't change bone map size 
						Section.BoneMap.RemoveAt(BoneMapIndex);
						ParentBoneMapIndex = Section.BoneMap.Add(*ParentBoneIndex);
					}

					// first fix up all indices of BoneMapRemapTable for the indices higher than BoneMapIndex, since BoneMapIndex is being removed
					for (auto Iter = BoneMapRemapTable.CreateIterator(); Iter; ++Iter)
					{
						FBoneIndexType& Value = Iter.Value();

						check (Value != BoneMapIndex);
						if (Value > BoneMapIndex)
						{
							--Value;
						}
					}

					int32 OldIndex = BoneMapIndex+AdjustIndex;
					int32 NewIndex = ParentBoneMapIndex;
					// you still have to add no matter what even if same since indices might change after added
					{
						// add to remap table
						check (OldIndex < FBoneIndexTypeMax && OldIndex >= 0);
						check (NewIndex < FBoneIndexTypeMax && NewIndex >= 0);
						check (BoneMapRemapTable.Contains((FBoneIndexType)OldIndex) == false);
						BoneMapRemapTable.Add((FBoneIndexType)OldIndex, (FBoneIndexType)NewIndex);
					}

					// reduce index since the item is removed
					--BoneMapIndex;
					--BoneMapSize;

					// this is to adjust the later indices. We need to refix their indices
					++AdjustIndex;
				}
				else if (AdjustIndex > 0)
				{
					int32 OldIndex = BoneMapIndex+AdjustIndex;
					int32 NewIndex = BoneMapIndex;
					check (OldIndex < FBoneIndexTypeMax && OldIndex >= 0);
					check (NewIndex < FBoneIndexTypeMax && NewIndex >= 0);
					check (BoneMapRemapTable.Contains((FBoneIndexType)OldIndex) == false);
					BoneMapRemapTable.Add((FBoneIndexType)OldIndex, (FBoneIndexType)NewIndex);
				}
			}

			if ( BoneMapRemapTable.Num() > 0 )
			{
				int32 BaseVertexIndex = Section.BaseVertexIndex;
				// fix up soft verts
				for (int32 VertIndex=0; VertIndex < Section.SoftVertices.Num(); ++VertIndex)
				{
					FSoftSkinVertex & Vert = Section.SoftVertices[VertIndex];

					auto RemapBoneInfluenceVertexIndex = [&BoneMapRemapTable](FBoneIndexType InfluenceBones[MAX_TOTAL_INFLUENCES], uint16 InfluenceWeights[MAX_TOTAL_INFLUENCES])
					{
						bool ShouldRenormalize = false;

						for (int32 InfluenceIndex = 0; InfluenceIndex < MAX_TOTAL_INFLUENCES; InfluenceIndex++)
						{
							FBoneIndexType *RemappedBone = BoneMapRemapTable.Find(InfluenceBones[InfluenceIndex]);
							if (RemappedBone)
							{
								InfluenceBones[InfluenceIndex] = *RemappedBone;
								ShouldRenormalize = true;
							}
						}

						if (ShouldRenormalize)
						{
							// should see if same bone exists
							for (int32 InfluenceIndex = 0; InfluenceIndex < MAX_TOTAL_INFLUENCES; InfluenceIndex++)
							{
								for (int32 InfluenceIndex2 = InfluenceIndex + 1; InfluenceIndex2 < MAX_TOTAL_INFLUENCES; InfluenceIndex2++)
								{
									// cannot be 0 because we don't allow removing root
									if (InfluenceBones[InfluenceIndex] != 0 && InfluenceBones[InfluenceIndex] == InfluenceBones[InfluenceIndex2])
									{
										InfluenceWeights[InfluenceIndex] += InfluenceWeights[InfluenceIndex2];
										// reset
										InfluenceBones[InfluenceIndex2] = 0;
										InfluenceWeights[InfluenceIndex2] = 0;
									}
								}
							}
						}
					};

					RemapBoneInfluenceVertexIndex(Vert.InfluenceBones, Vert.InfluenceWeights);

					int32 RealVertexIndex = BaseVertexIndex + VertIndex;
					//Remap the alternate weights
					for (auto Kvp : SkinWeightProfiles)
					{
						FImportedSkinWeightProfileData& SkinWeightProfile = SkinWeightProfiles.FindChecked(Kvp.Key);
						//If the vertex index is not handle by the runtime skin weight profile, the next skeletal mesh build will fix the runtime skin weight profile amount of vertices.
						if (SkinWeightProfile.SkinWeights.IsValidIndex(RealVertexIndex))
						{
							RemapBoneInfluenceVertexIndex(SkinWeightProfile.SkinWeights[RealVertexIndex].InfluenceBones, SkinWeightProfile.SkinWeights[RealVertexIndex].InfluenceWeights);
						}
					}
				}
			}
		}
	}

	void RetrieveBoneMatrices(USkeletalMesh* SkeletalMesh, const int32 LODIndex, const TArray<int32>& BonesToRemove, TArray<FMatrix>& InOutMatrices) const
	{
		if (!SkeletalMesh->IsValidLODIndex(LODIndex))
		{
			return;
		}

		// Retrieve all bone names in skeleton
		TArray<FName> BoneNames;
		const int32 NumBones = SkeletalMesh->GetRefSkeleton().GetNum();
		for (int32 BoneIndex = 0; BoneIndex < NumBones; ++BoneIndex)
		{
			BoneNames.Add(SkeletalMesh->GetRefSkeleton().GetBoneName(BoneIndex));
		}

		// get the relative to ref pose matrices
		TArray<FMatrix> RelativeToRefPoseMatrices;
		RelativeToRefPoseMatrices.AddDefaulted(NumBones);

		// Set initial matrices to identity
		for (int32 Index = 0; Index < NumBones; ++Index)
		{
			RelativeToRefPoseMatrices[Index] = FMatrix::Identity;
		}

		// if it has bake pose, gets ref to local matrices using bake pose
		if (const UAnimSequence* BakePoseAnim = SkeletalMesh->GetBakePose(LODIndex))
		{
			FMemMark Mark(FMemStack::Get());

			const FReferenceSkeleton& RefSkeleton = SkeletalMesh->GetRefSkeleton();


			// Setup BoneContainer and CompactPose
			TArray<FBoneIndexType> RequiredBoneIndexArray;
			RequiredBoneIndexArray.AddUninitialized(RefSkeleton.GetNum());
			{
				FBoneIndexType RequiredBoneIndexNum = IntCastChecked<FBoneIndexType>(RequiredBoneIndexArray.Num());
				for (FBoneIndexType BoneIndex = 0; BoneIndex < RequiredBoneIndexNum; ++BoneIndex)
				{
					RequiredBoneIndexArray[BoneIndex] = BoneIndex;
				}
			}
			FBoneContainer RequiredBones(RequiredBoneIndexArray, UE::Anim::ECurveFilterMode::DisallowAll, *SkeletalMesh);
			RequiredBones.SetUseRAWData(true);

			FCompactPose Pose;
			Pose.SetBoneContainer(&RequiredBones);
			Pose.ResetToRefPose();

			FBlendedCurve TempCurve;
			TempCurve.InitFrom(RequiredBones);
			UE::Anim::FStackAttributeContainer TempAttributes;

			FAnimationPoseData AnimPoseData(Pose, TempCurve, TempAttributes);

			// Get component space retarget base pose, will be equivalent of ref-pose if not edited
			TArray<FTransform> ComponentSpaceRefPose;
			FAnimationRuntime::FillUpComponentSpaceTransforms(SkeletalMesh->GetRefSkeleton(), SkeletalMesh->GetRefSkeleton().GetRefBonePose(), ComponentSpaceRefPose);
			
			// Retrieve animated pose from anim sequence (including retargeting)
			const USkeleton* Skeleton = SkeletalMesh->GetSkeleton();
			const FName RetargetSource = Skeleton->GetRetargetSourceForMesh(SkeletalMesh);
			UE::Anim::BuildPoseFromModel(BakePoseAnim->GetDataModel(), AnimPoseData, 0.0, EAnimInterpolationType::Step, RetargetSource, Skeleton->GetRefLocalPoses(RetargetSource));
			
			// Calculate component space animated pose matrices
			TArray<FMatrix> ComponentSpaceAnimatedPose;
			ComponentSpaceAnimatedPose.AddDefaulted(NumBones);

			for (int32 BoneIndex = 0; BoneIndex < NumBones; ++BoneIndex)
			{
				const FCompactPoseBoneIndex PoseBoneIndex(BoneIndex);
				const int32 ParentIndex = RefSkeleton.GetParentIndex(BoneIndex);
				if (ParentIndex != INDEX_NONE)
				{
					// If the bone will be removed, get the local-space retarget-ed animation bone transform
					if (BonesToRemove.Contains(BoneIndex))
					{
						ComponentSpaceAnimatedPose[BoneIndex] = Pose[PoseBoneIndex].ToMatrixWithScale() * ComponentSpaceAnimatedPose[ParentIndex];
					}
					// Otherwise use the component-space retarget base pose transform
					else
					{
						ComponentSpaceAnimatedPose[BoneIndex] = ComponentSpaceRefPose[BoneIndex].ToMatrixWithScale();
					}
				}
				else
				{
					// If the bone will be removed, get the retarget-ed animation bone transform
					if (BonesToRemove.Contains(BoneIndex))
					{
						ComponentSpaceAnimatedPose[BoneIndex] = Pose[PoseBoneIndex].ToMatrixWithScale();
					}
					// Otherwise use the retarget base pose transform
					else
					{
						ComponentSpaceAnimatedPose[BoneIndex] = ComponentSpaceRefPose[BoneIndex].ToMatrixWithScale();
					}
				}
			}

			// Calculate relative to retarget base (ref) pose matrix
			for (int32 BoneIndex = 0; BoneIndex < NumBones; ++BoneIndex)
			{
				RelativeToRefPoseMatrices[BoneIndex] = ComponentSpaceRefPose[BoneIndex].ToMatrixWithScale().Inverse() * ComponentSpaceAnimatedPose[BoneIndex];
			}
		}
		
		// Add bone transforms we're interested in
		InOutMatrices.Reset(BonesToRemove.Num());
		for (const int32 Index : BonesToRemove)
		{
			InOutMatrices.Add(RelativeToRefPoseMatrices[Index]);
		}
	}
	
	// Iterate through all LODs and add all of the sections' bones to the set.
	// OutBoneIndices will contain all bones that have vertices weighted against them.
	void FindBonesUsedBySkinWeights(const USkeletalMesh* SkeletalMesh, TSet<int32>& OutBoneIndices)
	{
		if (!SkeletalMesh)
		{
			return;
		}

		const FSkeletalMeshModel* ImportedModel = SkeletalMesh->GetImportedModel();
		if (!ImportedModel)
		{
			return;
		}
		
		OutBoneIndices.Reset();

		// Iterate through all LODs and add all of the sections' bones to the set.
		const int32 NumLODs = ImportedModel->LODModels.Num();
		for (int32 LODIndex = 0; LODIndex < NumLODs; ++LODIndex)
		{
			const FSkeletalMeshLODModel& LODModel = ImportedModel->LODModels[LODIndex];
			for (const FSkelMeshSection& Section : LODModel.Sections)
			{
				for (const int32 BoneIndex : Section.BoneMap)
				{
					if (BoneIndex != INDEX_NONE)
					{
						OutBoneIndices.Add(BoneIndex);
					}
				}
			}
		}
	}

	void AddAllParentBonesToRoot(const USkeletalMesh* SkeletalMesh, TSet<int32>& InOutBoneIndices)
	{
		if (!SkeletalMesh)
		{
			return;
		}

		const FReferenceSkeleton& ReferenceSkeleton = SkeletalMesh->GetRefSkeleton();
		if (ReferenceSkeleton.GetNum() == 0)
		{
			return;
		}

		const TSet<int32> Copy = InOutBoneIndices;
		for (int32 BoneIndex : Copy)
		{
			if (!ReferenceSkeleton.IsValidIndex(BoneIndex))
			{
				continue;
			}
			
			int32 ParentBoneIndex = ReferenceSkeleton.GetParentIndex(BoneIndex);
			while (ParentBoneIndex != INDEX_NONE)
			{
				InOutBoneIndices.Add(ParentBoneIndex);
				ParentBoneIndex = ReferenceSkeleton.GetParentIndex(ParentBoneIndex);
			}
		}
	}

	void AddBonesToKeepByName(const USkeletalMesh* SkeletalMesh, const TArray<FName>& ForceKeepBones, TSet<int32>& InOutKeepBones)
	{
		if (!SkeletalMesh || ForceKeepBones.Num() == 0)
		{
			return;
		}

		const FReferenceSkeleton& ReferenceSkeleton = SkeletalMesh->GetRefSkeleton();

		for (const FName BoneName : ForceKeepBones)
		{
			const int32 BoneIndex = ReferenceSkeleton.FindBoneIndex(BoneName);
			if (BoneIndex != INDEX_NONE)
			{
				InOutKeepBones.Add(BoneIndex);
			}
		}
	}
	
	void BuildBonesToBeRemoved(const USkeletalMesh* SkeletalMesh, const TSet<int32>& KeepBoneIndices, TArray<FName>& OutBonesToRemove)
	{
		if (!SkeletalMesh)
		{
			return;
		}
		
		const FReferenceSkeleton& ReferenceSkeleton = SkeletalMesh->GetRefSkeleton();
		const int32 NumBones = ReferenceSkeleton.GetNum();

		OutBonesToRemove.Reset();
		OutBonesToRemove.Reserve(NumBones);

		for (int32 BoneIndex = 0; BoneIndex < NumBones; ++BoneIndex)
		{
			if (!KeepBoneIndices.Contains(BoneIndex))
			{
				OutBonesToRemove.Add(ReferenceSkeleton.GetBoneName(BoneIndex));
			}
		}
	}
	
	void BuildBonesToBeRemovedUsedBySkinWeights(const USkeletalMesh* SkeletalMesh, const TArray<FName>& ForceKeepBones, TArray<FName>& OutBonesToRemove)
	{
		// Build the list of bones that we need to keep.
		TSet<int32> KeepBoneIndices;
		FindBonesUsedBySkinWeights(SkeletalMesh, KeepBoneIndices);
		AddBonesToKeepByName(SkeletalMesh, ForceKeepBones, KeepBoneIndices);
		AddAllParentBonesToRoot(SkeletalMesh, KeepBoneIndices);

		// Build the list of bones that we want to remove.
		BuildBonesToBeRemoved(SkeletalMesh, KeepBoneIndices, OutBonesToRemove);
	}
	
	void ReduceBoneCounts(USkeletalMesh* SkeletalMesh, const TArray<FName>& ForceKeepBones, const TArray<FName> BonesToRemove, int32 LODIndex, bool bIncludeBelowLODs)
	{
		if (BonesToRemove.Num() == 0)
		{
			return;
		}
		
		// Scoped post edit change
		FScopedSkeletalMeshPostEditChange ScopedPostEditChange(SkeletalMesh);
		
		// 1. Capture state before removal.
		const FReferenceSkeleton& RefSkel = SkeletalMesh->GetRefSkeleton();
		const int32 TotalRefBones = RefSkel.GetNum();
		int32 RequiredBonesBefore = 0;
		{
			if (const FSkeletalMeshModel* Model = SkeletalMesh->GetImportedModel())
			{
				if (Model->LODModels.IsValidIndex(LODIndex))
				{
					RequiredBonesBefore = Model->LODModels[LODIndex].RequiredBones.Num();
				}
			}
		}

		UE_LOGF(LogTemp, Display,
			"[ReduceBoneCounts] %ls: RefBones: %d | ForceKeep: %d | Removing: %d",
			*SkeletalMesh->GetName(),
			TotalRefBones,
			ForceKeepBones.Num(),
			BonesToRemove.Num());
	
		
		// 2. Bone removal
		for (const FName& BoneName : BonesToRemove)
		{
			SkeletalMesh->AddBoneToReductionSetting(LODIndex, BoneName);
		}
	
		constexpr bool bCallPostEditChange = false;
	
		if (bIncludeBelowLODs)
		{
			const int32 TotalLOD = SkeletalMesh->GetLODNum();
			for (int32 Index = LODIndex + 1; Index < TotalLOD; ++Index)
			{
				SkeletalMesh->AddBoneToReductionSetting(Index, BonesToRemove);
				ReduceBoneCounts(SkeletalMesh, Index, nullptr, bCallPostEditChange);
			}
		}

		ReduceBoneCounts(SkeletalMesh, LODIndex, nullptr, bCallPostEditChange);
		
		
		// 3. Capture state after removal.
		int32 RequiredBonesAfter = 0;
		TSet<int32> RequiredBoneSetAfter;

		if (const FSkeletalMeshModel* Model = SkeletalMesh->GetImportedModel())
		{
			if (Model->LODModels.IsValidIndex(LODIndex))
			{
				const FSkeletalMeshLODModel& LODModel = Model->LODModels[LODIndex];
				RequiredBonesAfter = LODModel.RequiredBones.Num();

				for (int32 BoneIndex : LODModel.RequiredBones)
				{
					RequiredBoneSetAfter.Add(BoneIndex);
				}
			}
		}

		// 4. Verification.
		for (const FName& RemovedBoneName : BonesToRemove)
		{
			const int32 RemovedIndex = RefSkel.FindBoneIndex(RemovedBoneName);
			if (RemovedIndex != INDEX_NONE && RequiredBoneSetAfter.Contains(RemovedIndex))
			{
				UE_LOGF(LogTemp, Warning,
					"[ReduceBoneCounts] %ls: Bone '%ls' could not be removed.",
					*SkeletalMesh->GetName(),
					*RemovedBoneName.ToString());
			}
		}

		const int32 RemovedCount = FMath::Max(0, RequiredBonesBefore - RequiredBonesAfter);

		UE_LOGF(LogTemp, Display, "[ReduceBoneCounts] %ls: Removed %d bones (RequiredBones: %d -> %d)",
			*SkeletalMesh->GetName(),
			RemovedCount,
			RequiredBonesBefore,
			RequiredBonesAfter);
		
		// Ensure the mesh gets regenerated. The bone hierarchy should be a part of the DDC key, but isn't, so we forcibly update the
		// base GUID instead.
		SkeletalMesh->InvalidateDeriveDataCacheGUID();
	}

	bool ReduceBoneCounts(USkeletalMesh* SkeletalMesh, int32 DesiredLOD, const TArray<FName>* BoneNamesToRemove, bool bCallPostEditChange /*= true*/) override
	{
		if (!SkeletalMesh)
		{
			UE_LOGF(LogMeshBoneReduction, Error, "Failed to remove Skeletal Mesh LOD %i bones, as the Skeletal Mesh is invalid.", DesiredLOD);
			return false;
		}

		if(!SkeletalMesh->GetSkeleton())
		{
			UE_LOGF(LogMeshBoneReduction, Error, "Failed to remove bones from LOD %i in %ls, as its Skeleton is invalid.", DesiredLOD, *SkeletalMesh->GetPathName());
			return false;
		}

		// find all the bones to remove from Skeleton settings
		TMap<FBoneIndexType, FBoneIndexType> BonesToRemove;

		bool bNeedsRemoval = GetBoneReductionData(SkeletalMesh, DesiredLOD, BonesToRemove, BoneNamesToRemove);
		// Always restore all previously removed bones if not contained by BonesToRemove
		SkeletalMesh->CalculateRequiredBones(SkeletalMesh->GetImportedModel()->LODModels[DesiredLOD], SkeletalMesh->GetRefSkeleton(), &BonesToRemove);
		
		if (IsInGameThread())
		{
			SkeletalMesh->ReleaseResources();
			SkeletalMesh->ReleaseResourcesFence.Wait();
		}
		else
		{
			// When building async, make sure the release resource has been made before starting the async task
			FSkeletalMeshRenderData* RenderData = SkeletalMesh->GetResourceForRendering();
			ensureMsgf(!RenderData || !RenderData->IsInitialized(), TEXT("Release Resource of async SkeletalMesh build must be done before going async!"));
		}

		FSkeletalMeshModel* SkeletalMeshResource = SkeletalMesh->GetImportedModel();
		check(SkeletalMeshResource);

		FSkeletalMeshLODModel** LODModels = SkeletalMeshResource->LODModels.GetData();
		FSkeletalMeshLODModel* SrcModel = LODModels[DesiredLOD];
		FSkeletalMeshLODModel* NewModel = nullptr;
				
		if (bNeedsRemoval)
		{
			NewModel = new FSkeletalMeshLODModel();
			LODModels[DesiredLOD] = NewModel;

			FSkeletalMeshLODModel::CopyStructure(NewModel, SrcModel);

			TArray<int32> BoneIndices;
			TArray<FMatrix> RemovedBoneMatrices;
			const bool bBakePoseToRemovedInfluences = (SkeletalMesh->GetBakePose(DesiredLOD) != nullptr);
			if (bBakePoseToRemovedInfluences)
			{
				for (const FBoneReference& BoneReference : SkeletalMesh->GetLODInfo(DesiredLOD)->BonesToRemove)
				{
					const int32 BoneIndex = SkeletalMesh->GetRefSkeleton().FindRawBoneIndex(BoneReference.BoneName);
					if (BoneIndex != INDEX_NONE)
					{
						BoneIndices.AddUnique(BoneIndex);
					}
				}

				for (const TPair<FBoneIndexType, FBoneIndexType>& BonePair : BonesToRemove)
				{
					if (BonePair.Key != INDEX_NONE)
					{
						BoneIndices.AddUnique(BonePair.Key);
					}
				}

				RetrieveBoneMatrices(SkeletalMesh, DesiredLOD, BoneIndices, RemovedBoneMatrices);
			}

			// fix up chunks
			ParallelFor(NewModel->Sections.Num(), [this, NewModel, bBakePoseToRemovedInfluences, &RemovedBoneMatrices, &BoneIndices, &BonesToRemove](const int32 SectionIndex)
			{
				FSkelMeshSection& Section = NewModel->Sections[SectionIndex];
				if (bBakePoseToRemovedInfluences)
				{
					using UE::AnimationCore::InvMaxRawBoneWeightFloat;
					
					for (FSoftSkinVertex& Vertex : Section.SoftVertices)
					{
						FVector TangentX = (FVector)Vertex.TangentX;
						FVector TangentY = (FVector)Vertex.TangentY;
						FVector TangentZ = (FVector)Vertex.TangentZ;
						FVector Position = (FVector)Vertex.Position;
						for (uint8 InfluenceIndex = 0; InfluenceIndex < MAX_TOTAL_INFLUENCES; ++InfluenceIndex)
				        {
							if (Vertex.InfluenceWeights[InfluenceIndex] > 0)
							{
								const int32 ArrayIndex = BoneIndices.IndexOfByKey(Section.BoneMap[Vertex.InfluenceBones[InfluenceIndex]]);
								if (ArrayIndex != INDEX_NONE)
								{
									Position += (FVector(RemovedBoneMatrices[ArrayIndex].TransformPosition((FVector)Vertex.Position)) - FVector(Vertex.Position)) * ((float)Vertex.InfluenceWeights[InfluenceIndex] * InvMaxRawBoneWeightFloat);

									TangentX += (FVector(RemovedBoneMatrices[ArrayIndex].TransformVector((FVector)Vertex.TangentX)) - FVector(Vertex.TangentX)) * ((float)Vertex.InfluenceWeights[InfluenceIndex] * InvMaxRawBoneWeightFloat);
									TangentY += (FVector(RemovedBoneMatrices[ArrayIndex].TransformVector((FVector)Vertex.TangentY)) - FVector(Vertex.TangentY)) * ((float)Vertex.InfluenceWeights[InfluenceIndex] * InvMaxRawBoneWeightFloat);
									TangentZ += (FVector(RemovedBoneMatrices[ArrayIndex].TransformVector((FVector)Vertex.TangentZ)) - FVector(Vertex.TangentZ)) * ((float)Vertex.InfluenceWeights[InfluenceIndex] * InvMaxRawBoneWeightFloat);
								}
							}
				        }

						Vertex.Position = (FVector3f)Position;
						Vertex.TangentX = (FVector3f)TangentX.GetSafeNormal();
						Vertex.TangentY = (FVector3f)TangentY.GetSafeNormal();
						const uint8 WComponent = static_cast<uint8>(Vertex.TangentZ.W);
						Vertex.TangentZ = (FVector3f)TangentZ.GetSafeNormal();
						Vertex.TangentZ.W = WComponent;
					}
				}
				FixUpSectionBoneMaps(Section, BonesToRemove, NewModel->SkinWeightProfiles);
			});

			// fix up RequiredBones/ActiveBoneIndices
			for (auto Iter = BonesToRemove.CreateIterator(); Iter; ++Iter)
			{
				FBoneIndexType BoneIndex = Iter.Key();
				FBoneIndexType MappingIndex = Iter.Value();
				NewModel->ActiveBoneIndices.Remove(BoneIndex);
				NewModel->RequiredBones.Remove(BoneIndex);

				NewModel->ActiveBoneIndices.AddUnique(MappingIndex);
				NewModel->RequiredBones.AddUnique(MappingIndex);
			}

			delete SrcModel;
		}
		else
		{
			NewModel = SrcModel;
		}		

		NewModel->ActiveBoneIndices.Sort();
		NewModel->RequiredBones.Sort();

		// Call post edit change and re-register skeletal mesh component
		if (bCallPostEditChange)
		{
			FScopedSkeletalMeshPostEditChange ScopedSkeletalMeshPostEditChange(SkeletalMesh);
		}

		if (IsInGameThread())
		{
			SkeletalMesh->MarkPackageDirty();
		}

		return true;
	}
};

TUniquePtr<FMeshBoneReduction> GMeshBoneReduction;

void FMeshBoneReductionModule::StartupModule()
{
	GMeshBoneReduction = MakeUnique<FMeshBoneReduction>();
}

void FMeshBoneReductionModule::ShutdownModule()
{
	GMeshBoneReduction = nullptr;
}

IMeshBoneReduction* FMeshBoneReductionModule::GetMeshBoneReductionInterface()
{
	return GMeshBoneReduction.Get();
}
