// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "BoneIndices.h"

class USkeletalMesh;
struct FSkelMeshSection;
struct FImportedSkinWeightProfileData;

/**
 * Mesh reduction interface.
 */
class IMeshBoneReduction
{
public:
	/**
	 * Fix up section bone maps based on list of bones to remove
	 * List of bones to remove should contains <bone index to remove, bone index to replace to >
	 * 
	 * @param	Section : Section reference to fix up
	 * @param	BonesToRemove : List of bones to remove with a pair of [bone index, bone to replace]
	 *
	 * @return	true if success 
	 */
	virtual void FixUpSectionBoneMaps( FSkelMeshSection & Section, const TMap<FBoneIndexType, FBoneIndexType> &BonesToRepair, TMap<FName, FImportedSkinWeightProfileData>& SkinWeightProfiles) = 0;

	/**
	 * Get Bones To Remove from the Desired LOD
	 * List of bones to remove should contains <bone index to remove, bone index to replace to >
	 *
	 * @param	SkeletalMesh : SkeletalMesh to test
	 * @param	DesiredLOD	: 0 isn't valid as this will only test from [LOD 1, LOD (N-1)] since Skeleton doesn't save any bones to remove setting on based LOD
	 * @param	OutBonesToReplace : List of bones to replace with a pair of [bone index, bone index to replace to]
	 *
	 * @return	true if any bone to be replaced
	 */
	virtual bool GetBoneReductionData(const USkeletalMesh* SkeletalMesh, int32 DesiredLOD, TMap<FBoneIndexType, FBoneIndexType> &OutBonesToReplace, const TArray<FName>* BoneNamesToRemove = NULL) = 0;

	/**
	 * Build a list of bone names that do not have vertices weighted against them.
	 *
	 * This function will iterate through all LODs and determines which bones are referenced by skin weights
	 * and adds these to the array of bones to be kept, along with the given bones to force keep as well
	 * as all parents of such bones up to the root bone. All remaining bones will be added to the output removal list. 
	 *
	 * The output can be used for bone count reduction for leaf skeletal meshes where we only need to process bones that
	 * actually influence any vertices. Don't use this for partly invisible skeletal meshes holding bones driving attachments.
	 *
	 * @param SkeletalMesh The skeletal mesh to analyze. If null, no bones will be removed.
	 * @param ForceKeepBones Optional list of bone names that must be preserved regardless of skin weight usage.
	 * @param OutBonesToRemove Output array populated with the names of bones that can be removed.
	 */
	virtual void BuildBonesToBeRemovedUsedBySkinWeights(const USkeletalMesh* SkeletalMesh, const TArray<FName>& ForceKeepBones, TArray<FName>& OutBonesToRemove) = 0;
	
	/**
	 * Reduce bone count based on the given list of bones to remove.
	 *
	 * This will remove the given set of bones from a LOD and possibly below while keeping the
	 * manually specified bones to keep as well as all the needed parents and parents of parents.
	 *
	 * The function updates derived data cache state to ensure regenerated skeletal data
	 * reflects the modified bone hierarchy.
	 *
	 * @param SkeletalMesh The skeletal mesh whose bone counts should be reduced.
	 * @param ForceKeepBones Optional list of bone names that must be preserved regardless.
	 * @param BonesToRemove List of bone names to remove from the specified LOD.
	 * @param LODIndex Index of the LOD to apply bone count reduction.
	 * @param bIncludeBelowLODs If true, applies the same bone removal to all LODs below the specified LOD index.
	 */
	virtual void ReduceBoneCounts(USkeletalMesh* SkeletalMesh, const TArray<FName>& ForceKeepBones, const TArray<FName> BonesToRemove, int32 LODIndex, bool bIncludeBelowLODs) = 0;
	
	/**
	 * Reduce Bone Counts for the SkeletalMesh with the LOD
	 *
	 * @param	SkeletalMesh : SkeletalMesh
	 * @param	DesriedLOD	: The data to reduce comes from Skeleton 
	 *
	 */
	virtual bool ReduceBoneCounts(USkeletalMesh* SkeletalMesh, int32 DesiredLOD, const TArray<FName>* BoneNamesToRemove, bool bCallPostEditChange = true) = 0;
};

/**
 * Mesh reduction module interface.
 */
class IMeshBoneReductionModule : public IModuleInterface
{
public:
	/**
	 * Retrieve the mesh reduction interface.
	 */
	virtual class IMeshBoneReduction* GetMeshBoneReductionInterface() = 0;
};
